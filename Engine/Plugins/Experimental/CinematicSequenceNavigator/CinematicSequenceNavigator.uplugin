{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Cinematic Sequence Navigator Bridge",
	"Description": "Sequence Navigator Bridge for Cinematic Assemblies",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "CinematicSequenceNavigator",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		}
	],
	"Plugins": [
		{
			"Name": "CinematicAssemblyTools",
			"Enabled": true,
			"Optional": true
		},
		{
			"Name": "SequenceNavigator",
			"Enabled": true,
			"Optional": true
		}
	]
}