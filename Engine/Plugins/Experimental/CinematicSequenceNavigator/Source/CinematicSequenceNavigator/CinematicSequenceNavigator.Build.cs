// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CinematicSequenceNavigator : ModuleRules
{
	public CinematicSequenceNavigator(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(new string[] {
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"CineAssemblyTools",
			"CoreUObject",
			"Engine",
			"MovieScene",
			"SequenceNavigator",
			"Sequencer",
			"SequencerCore",
			"Slate",
			"SlateCore",
			"ToolMenus",
			"UnrealEd"
		});
	}
}
