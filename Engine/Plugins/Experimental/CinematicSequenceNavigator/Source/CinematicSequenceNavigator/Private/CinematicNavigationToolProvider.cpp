// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicNavigationToolProvider.h"
#include "CineAssembly.h"
#include "CinematicNavigationToolProvider.h"
#include "CinematicSequenceNavigatorCommands.h"
#include "Columns/NavigationToolColorColumn.h"
#include "Columns/NavigationToolColumnExtender.h"
#include "Columns/NavigationToolCommentColumn.h"
#include "Columns/NavigationToolDeactiveStateColumn.h"
#include "Columns/NavigationToolHBiasColumn.h"
#include "Columns/NavigationToolIdColumn.h"
#include "Columns/NavigationToolInTimeColumn.h"
#include "Columns/NavigationToolItemsColumn.h"
#include "Columns/NavigationToolLabelColumn.h"
#include "Columns/NavigationToolLengthColumn.h"
#include "Columns/NavigationToolLockColumn.h"
#include "Columns/NavigationToolMarkerVisibilityColumn.h"
#include "Columns/NavigationToolOutTimeColumn.h"
#include "Columns/NavigationToolPlayheadColumn.h"
#include "Columns/NavigationToolRevisionControlColumn.h"
#include "Columns/NavigationToolStartFrameOffsetColumn.h"
#include "Columns/NavigationToolTakeColumn.h"
#include "Engine/Level.h"
#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "ISettingsModule.h"
#include "ISourceControlModule.h"
#include "ItemActions/NavigationToolAddItem.h"
#include "Items/NavigationToolItemParameters.h"
#include "Items/NavigationToolSequence.h"
#include "MovieScene.h"
#include "NavigationToolDefines.h"
#include "NavigationToolSettings.h"

#define LOCTEXT_NAMESPACE "CinematicNavigationToolProvider"

namespace UE::CineAssemblyTools
{

const FName FCinematicNavigationToolProvider::Identifier = TEXT("CinematicAssembly");

const FText FCinematicNavigationToolProvider::CinematicColumnViewName = LOCTEXT("CinematicAssemblyColumnViewName", "Cinematic Assembly");

const FName FCinematicNavigationToolProvider::ToolbarSectionName = TEXT("CinematicAssembly");
const FName FCinematicNavigationToolProvider::ContextMenuSectionName = TEXT("CinematicAssemblyActions");

FCinematicNavigationToolProvider::FCinematicNavigationToolProvider(const TSharedRef<ISequencer>& InSequencer)
	: WeakSequencer(InSequencer)
	, ToolCommands(MakeShared<FUICommandList>())
{
}

FName FCinematicNavigationToolProvider::GetIdentifier() const
{
	return Identifier;
}

TSet<TSubclassOf<UMovieSceneSequence>> FCinematicNavigationToolProvider::GetSupportedSequenceClasses() const
{
	return { UCineAssembly::StaticClass() };
}

FText FCinematicNavigationToolProvider::GetDefaultColumnView() const
{
	return CinematicColumnViewName;
}

FMovieSceneEditorData* FCinematicNavigationToolProvider::GetRootMovieSceneEditorData(const UE::SequenceNavigator::INavigationTool& InTool) const
{
	const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	UMovieSceneSequence* const RootMovieSceneSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootMovieSceneSequence)
	{
		return nullptr;
	}

	UMovieScene* const RootMovieScene = RootMovieSceneSequence->GetMovieScene();
	if (!RootMovieScene)
	{
		return nullptr;
	}

	return &RootMovieScene->GetEditorData();
}

FNavigationToolSaveState* FCinematicNavigationToolProvider::GetSaveState(const UE::SequenceNavigator::INavigationTool& InTool) const
{
	if (FMovieSceneEditorData* const EditorData = GetRootMovieSceneEditorData(InTool))
	{
		return &EditorData->NavigationToolState;
	}
	return nullptr;
}

void FCinematicNavigationToolProvider::SetSaveState(const UE::SequenceNavigator::INavigationTool& InTool
	, const FNavigationToolSaveState& InSaveState) const
{
	if (FMovieSceneEditorData* const EditorData = GetRootMovieSceneEditorData(InTool))
	{
		EditorData->NavigationToolState = InSaveState;
	}
}

void FCinematicNavigationToolProvider::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	InCommandList->Append(ToolCommands);

	//@TODO: Bind commands to functions
}

void FCinematicNavigationToolProvider::OnActivate()
{
	ExtendToolToolBar();
	ExtendToolItemContextMenu();
}

void FCinematicNavigationToolProvider::OnDeactivate()
{
	RemoveToolToolBarExtension();
	RemoveToolItemContextMenuExtension();
}

void FCinematicNavigationToolProvider::OnExtendColumns(UE::SequenceNavigator::FNavigationToolColumnExtender& OutExtender)
{
	using namespace UE::SequenceNavigator;

	// Support built in columns
	OutExtender.AddColumn<FNavigationToolPlayheadColumn>();
	OutExtender.AddColumn<FNavigationToolDeactiveStateColumn>();
	OutExtender.AddColumn<FNavigationToolMarkerVisibilityColumn>();
	OutExtender.AddColumn<FNavigationToolLockColumn>();
	OutExtender.AddColumn<FNavigationToolColorColumn>();
	OutExtender.AddColumn<FNavigationToolLabelColumn>();
	OutExtender.AddColumn<FNavigationToolIdColumn>();
	OutExtender.AddColumn<FNavigationToolItemsColumn>();
	OutExtender.AddColumn<FNavigationToolInTimeColumn>();
	OutExtender.AddColumn<FNavigationToolOutTimeColumn>();
	OutExtender.AddColumn<FNavigationToolLengthColumn>();
	OutExtender.AddColumn<FNavigationToolHBiasColumn>();
	OutExtender.AddColumn<FNavigationToolStartFrameOffsetColumn>();
	OutExtender.AddColumn<FNavigationToolTakeColumn>();
	OutExtender.AddColumn<FNavigationToolCommentColumn>();

	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled())
	{
		OutExtender.AddColumn<FNavigationToolRevisionControlColumn>();
	}

	//@TODO: Add Cinematic Assembly specific columns

	FNavigationToolProvider::OnExtendColumns(OutExtender);
}

void FCinematicNavigationToolProvider::OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews)
{
	using namespace UE::SequenceNavigator;

	FNavigationToolColumnView& CinematicColumnView = OutColumnViews
		.FindOrAdd(FNavigationToolColumnView(CinematicColumnViewName));

	CinematicColumnView.VisibleColumns =
		{
			FNavigationToolPlayheadColumn::StaticColumnId(),
			FNavigationToolColorColumn::StaticColumnId(),
			FNavigationToolLabelColumn::StaticColumnId(),
			FNavigationToolItemsColumn::StaticColumnId(),
			FNavigationToolInTimeColumn::StaticColumnId(),
			FNavigationToolOutTimeColumn::StaticColumnId(),
			FNavigationToolHBiasColumn::StaticColumnId()
		};

	FNavigationToolProvider::OnExtendColumnViews(OutColumnViews);
}

void FCinematicNavigationToolProvider::OnExtendItemChildren(SequenceNavigator::INavigationTool& InTool
	, const SequenceNavigator::FNavigationToolViewModelPtr& InParentItem
	, TArray<SequenceNavigator::FNavigationToolViewModelWeakPtr>& OutWeakChildren
	, const bool bInRecursive)
{
	using namespace UE::SequenceNavigator;

	FNavigationToolProvider::OnExtendItemChildren(InTool, InParentItem, OutWeakChildren, bInRecursive);

	// Only extending root item
	if (InParentItem->GetItemId() != FNavigationToolItemId::RootId)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const RootSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return;
	}

	const TSharedRef<FCinematicNavigationToolProvider> ProviderRef = SharedThis(this);

	const FNavigationToolViewModelPtr NewItem = InTool.FindOrAdd<FNavigationToolSequence>(ProviderRef
		, InParentItem, RootSequence, nullptr, 0);
	const FNavigationToolItemFlagGuard Guard(NewItem, ENavigationToolItemFlags::IgnorePendingKill);
	OutWeakChildren.Add(NewItem);
	if (bInRecursive)
	{
		NewItem->FindChildren(OutWeakChildren, bInRecursive);
	}
}

void FCinematicNavigationToolProvider::OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams)
{
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateSequenceFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateTrackFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateBindingFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateMarkerFilter());

	FNavigationToolProvider::OnExtendBuiltInFilters(OutFilterParams);
}

TSharedPtr<SequenceNavigator::INavigationTool> FCinematicNavigationToolProvider::GetNavigationTool() const
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return SequenceNavigator::FNavigationToolExtender::FindNavigationTool(Sequencer.ToSharedRef());
	}
	return nullptr;
}

void FCinematicNavigationToolProvider::ExtendToolToolBar()
{
	//@TODO: Add navigator toolbar buttons
}

void FCinematicNavigationToolProvider::ExtendToolItemContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	UToolMenu* const ExtendedMenu = ToolMenus->ExtendMenu(UE::SequenceNavigator::GetItemContextMenuName());
	if (!ExtendedMenu)
	{
		return;
	}

	//const FCinematicSequenceNavigatorCommands& CinematicSequenceNavigatorCommands = FCinematicSequenceNavigatorCommands::Get();

	//@TODO: Extend item context menu
}

void FCinematicNavigationToolProvider::RemoveToolToolBarExtension()
{
	if (UToolMenu* const Menu = UToolMenus::Get()->FindMenu(UE::SequenceNavigator::GetToolBarMenuName()))
	{
		Menu->RemoveSection(ToolbarSectionName);
	}
}

void FCinematicNavigationToolProvider::RemoveToolItemContextMenuExtension()
{
	if (UToolMenu* const Menu = UToolMenus::Get()->FindMenu(UE::SequenceNavigator::GetItemContextMenuName()))
	{
		Menu->RemoveSection(ContextMenuSectionName);
	}
}

void FCinematicNavigationToolProvider::ExtendToolItemDragDropOp(UE::SequenceNavigator::FNavigationToolItemDragDropOp& InDragDropOp)
{
	//@TODO: Extend drag and drop operations
}

} // namespace UE::CineAssemblyTools

#undef LOCTEXT_NAMESPACE
