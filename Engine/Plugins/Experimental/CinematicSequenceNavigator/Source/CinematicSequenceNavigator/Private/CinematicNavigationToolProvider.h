// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "NavigationToolDefines.h"
#include "Providers/NavigationToolProvider.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FUICommandList;
class UMovieSceneSequence;
class UObject;
struct FNavigationToolSaveState;
struct FMovieSceneEditorData;

namespace UE::SequenceNavigator
{
	class FNavigationToolAvaSequence;
	class FNavigationToolItem;
	class FNavigationToolItemDragDropOp;
	class INavigationTool;
}

namespace UE::CineAssemblyTools
{

class FCinematicNavigationToolProvider : public SequenceNavigator::FNavigationToolProvider
{
public:
	static const FName Identifier;

	static const FText CinematicColumnViewName;

	FCinematicNavigationToolProvider(const TSharedRef<ISequencer>& InSequencer);
	virtual ~FCinematicNavigationToolProvider() override = default;

	//~ Begin INavigationToolProvider

	virtual FName GetIdentifier() const override;

	virtual TSet<TSubclassOf<UMovieSceneSequence>> GetSupportedSequenceClasses() const override;

	virtual FText GetDefaultColumnView() const override;

	virtual FNavigationToolSaveState* GetSaveState(const SequenceNavigator::INavigationTool& InTool) const override;
	virtual void SetSaveState(const SequenceNavigator::INavigationTool& InTool, const FNavigationToolSaveState& InSaveState) const override;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;

	virtual void OnActivate() override;
	virtual void OnDeactivate() override;

	virtual void OnExtendColumns(SequenceNavigator::FNavigationToolColumnExtender& OutExtender) override;
	virtual void OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews) override;
	virtual void OnExtendItemChildren(SequenceNavigator::INavigationTool& InTool
		, const SequenceNavigator::FNavigationToolViewModelPtr& InParentItem
		, TArray<SequenceNavigator::FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) override;
	virtual void OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams) override;

	//~ End INavigationToolProvider

private:
	static const FName ToolbarSectionName;
	static const FName ContextMenuSectionName;

	TSharedPtr<SequenceNavigator::INavigationTool> GetNavigationTool() const;

	void ExtendToolToolBar();
	void ExtendToolItemContextMenu();

	void RemoveToolToolBarExtension();
	void RemoveToolItemContextMenuExtension();

	void ExtendToolItemDragDropOp(SequenceNavigator::FNavigationToolItemDragDropOp& InDragDropOp);

	FMovieSceneEditorData* GetRootMovieSceneEditorData(const UE::SequenceNavigator::INavigationTool& InTool) const;

	TWeakPtr<ISequencer> WeakSequencer;

	TSharedRef<FUICommandList> ToolCommands;
};

} // namespace UE::CineAssemblyTools
