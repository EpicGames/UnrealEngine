// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FCinematicSequenceNavigatorCommands : public TCommands<FCinematicSequenceNavigatorCommands>
{
public:
	FCinematicSequenceNavigatorCommands()
		: TCommands<FCinematicSequenceNavigatorCommands>(TEXT("CinematicSequenceNavigatorCommands")
		, NSLOCTEXT("CinematicSequenceNavigatorCommands"
			, "CinematicSequenceNavigatorCommands", "Cinematic Sequence Navigator Commands")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	// @TODO: Add commands here
};
