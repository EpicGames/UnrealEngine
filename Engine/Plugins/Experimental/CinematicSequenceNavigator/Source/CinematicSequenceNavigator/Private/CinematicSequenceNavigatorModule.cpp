// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicSequenceNavigatorModule.h"
#include "CinematicNavigationToolProvider.h"
#include "CinematicSequenceNavigatorCommands.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "NavigationToolExtender.h"

#define LOCTEXT_NAMESPACE "FCinematicSequenceNavigatorModule"

using namespace UE::CineAssemblyTools;
using namespace UE::SequenceNavigator;

void FCinematicSequenceNavigatorModule::StartupModule()
{
	FCinematicSequenceNavigatorCommands::Register();

	if (ISequencerModule* const SequencerModule = FModuleManager::LoadModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerCreatedHandle = SequencerModule->RegisterOnSequencerCreated(
			FOnSequencerCreated::FDelegate::CreateRaw(this, &FCinematicSequenceNavigatorModule::OnSequencerCreated));
	}
}

void FCinematicSequenceNavigatorModule::ShutdownModule()
{
	if (ISequencerModule* const SequencerModule = FModuleManager::LoadModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModule->UnregisterOnSequencerCreated(SequencerCreatedHandle);
		SequencerCreatedHandle.Reset();
	}

	FCinematicSequenceNavigatorCommands::Unregister();
}

void FCinematicSequenceNavigatorModule::OnSequencerCreated(const TSharedRef<ISequencer> InSequencer)
{
	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*InSequencer);

	const TSharedPtr<FNavigationToolProvider> FoundProvider = FNavigationToolExtender::FindToolProvider(ToolId
		, FCinematicNavigationToolProvider::Identifier);
	if (!FoundProvider.IsValid())
	{
		NavigationToolProvider = MakeShared<FCinematicNavigationToolProvider>(InSequencer);
		if (NavigationToolProvider->IsSequenceSupported(InSequencer->GetRootMovieSceneSequence()))
		{
			FNavigationToolExtender::RegisterToolProvider(InSequencer, NavigationToolProvider.ToSharedRef());
		}
		else
		{
			NavigationToolProvider.Reset();
		}
	}

	SequencerClosedHandle = InSequencer->OnCloseEvent().AddRaw(this, &FCinematicSequenceNavigatorModule::OnSequencerClosed);
}

void FCinematicSequenceNavigatorModule::OnSequencerClosed(const TSharedRef<ISequencer> InSequencer)
{
	if (!NavigationToolProvider.IsValid())
	{
		return;
	}

	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*InSequencer);

	if (FNavigationToolExtender::UnregisterToolProvider(ToolId, NavigationToolProvider->GetIdentifier()))
	{
		NavigationToolProvider.Reset();
		SequencerClosedHandle.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCinematicSequenceNavigatorModule, CinematicSequenceNavigator)
